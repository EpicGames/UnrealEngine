// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MoveLibrary/RollbackBlackboardWrappers.h"
#include "MoverTypes.h"
#include "MoverSimulationTypes.h"
#include "Misc/LowLevelTestAdapter.h"
#include "Misc/AutomationTest.h"
#include "EngineRuntimeTests.h"

static int FixedMsPerFrame = 10;
static void AdvanceTimeStepFrame(FMoverTimeStep& TimeStepToModify)
{
	++TimeStepToModify.ServerFrame;
	TimeStepToModify.BaseSimTimeMs += FixedMsPerFrame;
	TimeStepToModify.StepMs = FixedMsPerFrame;
}

// Simple test cases
TEST_CASE_NAMED(FRollbackBlackboardTest, "Mover::RollbackBlackboard::TestOperations", "[Mover][Blackboard]")
{
	TObjectPtr<URollbackBlackboard> RBBB = NewObject<URollbackBlackboard>(GetTransientPackage(), TEXT("TestRollbackBlackboard"), RF_Transient);

	REQUIRE(RBBB != nullptr);

	FRollbackBlackboardSimWrapper RBBBSimAccessor(*RBBB, /*bPredictiveMode*/ false);
	FRollbackBlackboardExternalWrapper RBBBExternalAccessor(*RBBB);

	FMoverTimeStep CurrentSimTime;
	CurrentSimTime.ServerFrame = 22;
	CurrentSimTime.BaseSimTimeMs = 300;
	CurrentSimTime.StepMs = 10;

	{
		URollbackBlackboard::EntrySettings ModeChangeRecordSettings;
		ModeChangeRecordSettings.SizingPolicy = EBlackboardSizingPolicy::FixedDeclaredSize;
		ModeChangeRecordSettings.FixedSize = 4;
		ModeChangeRecordSettings.PersistencePolicy = EBlackboardPersistencePolicy::Forever;
		ModeChangeRecordSettings.RollbackPolicy = EBlackboardRollbackPolicy::InvalidatedOnRollback;

		RBBBSimAccessor.CreateEntry<FMovementModeChangeRecord>(CommonBlackboard::LastModeChangeRecord, ModeChangeRecordSettings);
	}

	{
		URollbackBlackboard::EntrySettings Integer32TestSettings;
		Integer32TestSettings.SizingPolicy = EBlackboardSizingPolicy::FixedDeclaredSize;
		Integer32TestSettings.FixedSize = 6;
		Integer32TestSettings.PersistencePolicy = EBlackboardPersistencePolicy::Forever;
		Integer32TestSettings.RollbackPolicy = EBlackboardRollbackPolicy::InvalidatedOnRollback;

		bool bWasGetSuccessful, bWasSetSuccessful;

		// Test that a value can be created, set, and get inside a simulation
		int32 Int32Val = -1;
		static const FName Integer32TestEntryName = TEXT("Integer32TestEntry");

		// Test: premature read before the element has been created
		bWasGetSuccessful = RBBBExternalAccessor.TryGet(Integer32TestEntryName, Int32Val);

		CHECK_EQUALS(TEXT("Verify premature external retrieval of value, before creation, should fail"), bWasGetSuccessful, false);


		RBBBSimAccessor.CreateEntry<int32>(Integer32TestEntryName, Integer32TestSettings);

		{
			RBBB->BeginSimulationFrame(CurrentSimTime);
		

			bWasSetSuccessful = RBBBSimAccessor.TrySet(Integer32TestEntryName, 32);
			bWasGetSuccessful = RBBBSimAccessor.TryGet(Integer32TestEntryName, Int32Val);

			CHECK_EQUALS(TEXT("Verify in-simulation setting of a value indicates success"), bWasSetSuccessful, true);
			CHECK_EQUALS(TEXT("Verify in-simulation getting of a value indicates success"), bWasGetSuccessful, true);
			CHECK_EQUALS(TEXT("Verify in-simulation getting of a value matches"), Int32Val, 32);

			// Test that the value CANNOT yet be retrieved from an external accessor
			bWasGetSuccessful = RBBBExternalAccessor.TryGet(Integer32TestEntryName, Int32Val);
			CHECK_EQUALS(TEXT("Verify premature external retrieval of value, before end of first sim frame, should fail"), bWasGetSuccessful, false);

			RBBB->EndSimulationFrame();
		}

		// Test that a value can now be retrieved externally
		bWasGetSuccessful = RBBBExternalAccessor.TryGet(Integer32TestEntryName, Int32Val);

		CHECK_EQUALS(TEXT("Verify external getting operation success"), bWasGetSuccessful, true);
		CHECK_EQUALS(TEXT("Verify external getting value matches"), Int32Val, 32);

		{
			AdvanceTimeStepFrame(CurrentSimTime);
			RBBB->BeginSimulationFrame(CurrentSimTime);

			// Test that the value can be retrieved externally while a sim is ongoing, but that the prior value is returned
			bWasSetSuccessful = RBBBSimAccessor.TrySet(Integer32TestEntryName, 64);

			bWasGetSuccessful = RBBBExternalAccessor.TryGet(Integer32TestEntryName, Int32Val);

			CHECK_EQUALS(TEXT("Verify external getting value remains the same from the prior frame until the sim frame ends"), Int32Val, 32);


			RBBB->EndSimulationFrame();

			bWasGetSuccessful = RBBBExternalAccessor.TryGet(Integer32TestEntryName, Int32Val);

			CHECK_EQUALS(TEXT("Verify external getting value reads the most recent value after the sim frame ends"), Int32Val, 64);

		}


		// Test predictive use
		AdvanceTimeStepFrame(CurrentSimTime);

		FRollbackBlackboardSimWrapper RBBBPredictiveAccessor(*RBBB, /*bPredictiveMode*/ true);

		const int32 NumFramesToPredict = 10;

		FMoverTimeStep PredictionTime = CurrentSimTime;

		int32 ExternalReadValue = -1;
		int32 EarlyPredictReadValue = -2;
		int32 LatePredictReadValue = -3;
		int32 FinalPredictReadValue = -4;


		const int32 LastNonPredictedValue = 222;
		RBBBExternalAccessor.TrySet(Integer32TestEntryName, LastNonPredictedValue);

		int32 LastWrittenPredictValue = LastNonPredictedValue;


		for (int32 i=0; i < NumFramesToPredict; ++i)
		{
			AdvanceTimeStepFrame(PredictionTime);

			bool bDidRead(false), bDidWrite(false);

			// Check that EXTERNAL reader is never polluted by predicted writes
			bDidRead = RBBBExternalAccessor.TryGet(Integer32TestEntryName, ExternalReadValue);
			CHECK(bDidRead);
			CHECK_EQUALS(TEXT("Verify external getting value reads the most recent true value (not predicted, early)"), ExternalReadValue, LastNonPredictedValue);

			RBBB->BeginPredictionFrame(PredictionTime);
			{
				bDidRead = RBBBPredictiveAccessor.TryGet(Integer32TestEntryName, EarlyPredictReadValue);
				CHECK(bDidRead);
				CHECK_EQUALS(TEXT("Verify early predicted read the most recent true value (predicted, early-prediction)"), EarlyPredictReadValue, LastWrittenPredictValue);

				// Advance predicted value and write it
				LastWrittenPredictValue = LastWrittenPredictValue + (i*10);
				bDidWrite = RBBBPredictiveAccessor.TrySet(Integer32TestEntryName, LastWrittenPredictValue);	// predictive SET
				CHECK(bDidWrite);

				// Test that the predictive read is the new value but the external accessor is unaffected
				bDidRead = RBBBPredictiveAccessor.TryGet(Integer32TestEntryName, LatePredictReadValue);
				CHECK(bDidRead);
				CHECK_EQUALS(TEXT("Verify late predicted read gets the most recent predicted value (predicted, late-prediction)"), LatePredictReadValue, LastWrittenPredictValue);

				bDidRead = RBBBExternalAccessor.TryGet(Integer32TestEntryName, ExternalReadValue);
				CHECK(bDidRead);
				CHECK_EQUALS(TEXT("Verify external getting value reads the most recent true value (not predicted, late-prediction)"), ExternalReadValue, LastNonPredictedValue);
			}

			bDidRead = RBBBPredictiveAccessor.TryGet(Integer32TestEntryName, FinalPredictReadValue);
			CHECK(bDidRead);
			CHECK_EQUALS(TEXT("Verify late predicted read gets the final predicted value (predicted, post-prediction)"), FinalPredictReadValue, LastWrittenPredictValue);

			bDidRead = RBBBExternalAccessor.TryGet(Integer32TestEntryName, ExternalReadValue);
			CHECK(bDidRead);
			CHECK_EQUALS(TEXT("Verify external getting value reads the most recent true value (not predicted, post-predicted frame)"), ExternalReadValue, LastNonPredictedValue);

		}

		RBBB->EndPrediction();
	}
}
