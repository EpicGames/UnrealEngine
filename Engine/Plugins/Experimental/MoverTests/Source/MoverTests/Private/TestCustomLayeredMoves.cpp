// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestCustomLayeredMoves.h"
#include "MoverComponent.h"
#include "MoverLog.h"

FTestCustomLayeredMove::FTestCustomLayeredMove()
{
	DurationMs = 0.f;
	MixMode = EMoveMixMode::OverrideVelocity;
}

bool FTestCustomLayeredMove::GenerateMove(const FMoverTickStartData& SimState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	OutProposedMove.MixMode = MixMode;
	OutProposedMove.LinearVelocity = LaunchVelocity;
	OutProposedMove.PreferredMode = ForceMovementMode;

	return true;
}

void FTestCustomLayeredMove::OnStart(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard)
{
	UE_LOGF(LogMover, Log, "Custom test layered move started!");
}

void FTestCustomLayeredMove::OnEnd(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs)
{
	UE_LOGF(LogMover, Log, "Custom test layered move ended!");
}

FLayeredMoveBase* FTestCustomLayeredMove::Clone() const
{
	FTestCustomLayeredMove* CopyPtr = new FTestCustomLayeredMove(*this);
	return CopyPtr;
}

void FTestCustomLayeredMove::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	SerializePackedVector<10, 16>(LaunchVelocity, Ar);

	bool bUsingForcedMovementMode = !ForceMovementMode.IsNone();

	Ar.SerializeBits(&bUsingForcedMovementMode, 1);

	if (bUsingForcedMovementMode)
	{
		Ar << ForceMovementMode;
	}

}

UScriptStruct* FTestCustomLayeredMove::GetScriptStruct() const
{
	return FTestCustomLayeredMove::StaticStruct();
}

FString FTestCustomLayeredMove::ToSimpleString() const
{
	return FString::Printf(TEXT("Custom Test Move"));
}

void FTestCustomLayeredMove::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}