// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LayeredMove.h"
#include "TestCustomLayeredMoves.generated.h"

/**
 * Custom Layered move for testing purposes - acts the same as the Launch layered move
 */
USTRUCT(BlueprintType)
struct MOVERTESTS_API FTestCustomLayeredMove : public FLayeredMoveBase
{
	GENERATED_USTRUCT_BODY()

	FTestCustomLayeredMove();
	virtual ~FTestCustomLayeredMove() override {}
	
	// Velocity to apply to the actor. Could be additive or overriding depending on MixMode setting.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(ForceUnits="cm/s"))
	FVector LaunchVelocity = FVector::ZeroVector;

	// Optional movement mode name to force the actor into before applying the impulse velocity.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FName ForceMovementMode = NAME_None;

	virtual void OnStart(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard) override;
	
	virtual void OnEnd(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs) override;
	
	// Generate a movement 
	virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	virtual FLayeredMoveBase* Clone() const override;

	virtual void NetSerialize(FArchive& Ar) override;

	virtual UScriptStruct* GetScriptStruct() const override;

	virtual FString ToSimpleString() const override;

	virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};